[CrashReportClient]
bAgreeToCrashUpload=false

[/Script/Engine.RendererSettings]
; Substrate settings for more advanced shading on PC and editor by default.
r.Substrate.RoughDiffuse=1
r.Substrate.ShadingQuality=1
r.Substrate.TileCoord8bits=0
r.Substrate.Glints=1
r.Substrate.ClosuresPerPixel=4
r.Refraction.Blur=1
