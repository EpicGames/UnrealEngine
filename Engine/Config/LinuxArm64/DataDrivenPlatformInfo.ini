[DataDrivenPlatformInfo]
IniParent=Unix
TargetSettingsIniSectionName=/Script/LinuxTargetPlatform.LinuxTargetSettings
bIsConfidential=false
bHasDedicatedGamepad=false
bDefaultInputStandardKeyboard=true
bInputSupportConfigurable=true
DefaultInputType=MouseAndKeyboard
bSupportsMouseAndKeyboard=true
bSupportsGamepad=true
bCanChangeGamepadType=true
bSupportsTouch=false
GlobalIdentifier=C0B45A339DE340C7BCE24C4715C3BABF

OverrideCookPlatformName=Linux

NormalIconPath=Launcher/Linux/Platform_Linux_24x
LargeIconPath=Launcher/Linux/Platform_Linux_128x
XLargeIconPath=
AutoSDKPath=Linux_x64
TutorialPath=SharingAndReleasing/Linux
Windows:bIsEnabled=true
Mac:bIsEnabled=false
Linux:bIsEnabled=true
Windows:bUsesHostCompiler=false
Linux:bUsesHostCompiler=true
bUATClosesAfterLaunch=true
PlatformGroupName=Mobile
PlatformSubMenu=Linux
AudioCompressionSettingsIniSectionName=/Script/LinuxTargetPlatform.LinuxTargetSettings
ICUDataVersion=icudt64l
