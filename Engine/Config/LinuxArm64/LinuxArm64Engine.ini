[Audio]
; Defines a platform-specific volume headroom (in dB) for audio to provide better platform consistency with respect to volume levels.
PlatformHeadroomDB=0

[SystemSettings]
r.setres=1280x720
;Perplatform to PerQualityLevel conversion mapping for platform groups
QualityLevelMapping="low"

;Manual range settings for Niagara quality levels. Simplifies handling of device profiles/fragments and what quality levels they can set.
;Please ensure device profiles or fragments for this platform do not set fx.Niagara.QualityLevel values outside of this range
fx.Niagara.QualityLevel.Min=0
fx.Niagara.QualityLevel.Max=0

[TextureStreaming]
; PoolSizeVRAMPercentage is how much percentage of GPU Dedicated VRAM should be used as a TexturePool cache for streaming textures (0 - unlimited streaming)
PoolSizeVRAMPercentage=70

[DeviceProfileManager]
DeviceProfileSelectionModule="LinuxDeviceProfileSelector"

[ConsoleVariables]
; larger timeout since drivers may take longer time
g.TimeoutForBlockOnRenderFence=60000

[PlatformCrypto]
PlatformRequiresDataCrypto=True

[ShaderPlatformConfig VULKAN_SM6]
; Bindless is necessary for Vulkan ray tracing
BindlessConfiguration=All